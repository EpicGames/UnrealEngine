// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPRolesModule.h"
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

DEFINE_LOG_CATEGORY(LogVPRoles);

IMPLEMENT_MODULE(FDefaultModuleImpl, VPRoles)
