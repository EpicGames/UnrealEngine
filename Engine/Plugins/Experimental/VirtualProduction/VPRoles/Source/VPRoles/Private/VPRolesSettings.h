// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayTagContainer.h"
#include "VPRolesSettings.generated.h"

UCLASS(config=UserVPRoles)
class VPROLES_API UVPRolesUserSettings : public UObject
{
public:
	GENERATED_BODY()

private:
	/**
	 * The machine role(s) in a virtual production context.
	 * @note The role may be override via the command line, "-VPRole=[Role.SubRole1|Role.SubRole2]"
	 */
	UPROPERTY(config)
	FGameplayTagContainer Roles;
	
	/** The files that contain the available VP Roles. */
	UPROPERTY(config)
	TSet<FName> RoleSources;
	
	/** Cached available roles since the list isn't available outside of editor. */
	UPROPERTY(config)
	TArray<FString> CachedRoles;
	
	friend class UVirtualProductionRolesSubsystem;
};
