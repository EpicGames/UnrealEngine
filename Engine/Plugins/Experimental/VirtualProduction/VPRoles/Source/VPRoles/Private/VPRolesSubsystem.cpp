// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPRolesSubsystem.h"

#include "Algo/Transform.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"
#include "Misc/CommandLine.h"
#include "VPRolesSettings.h"
#include "VPRolesModule.h"
#include "VPSettings.h"

#if WITH_EDITOR
#include "Editor.h"
#include "GameplayTagsEditorModule.h"
#endif

static const FName DefaultVPRolesFileName = ("VPRoles.ini");

void UVirtualProductionRolesSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	CommandLineRoles = MakePimpl<FGameplayTagContainer>();
	ImportCurrentRolesFromCommandLine();
	ImportCurrentRolesFromDeprecatedSetting();

	AddRoleSource(DefaultVPRolesFileName);

	// todo: Pop up menu to migrate all sources to the VPRoles one.
	ImportRoleSourcesFromVPSettings();
	
	// Provide a getter to the old system so it can return up to date roles.
	GetMutableDefault<UVPSettings>()->RolesGetter.BindUObject(this, &UVirtualProductionRolesSubsystem::GetRolesContainer_Private);
}

void UVirtualProductionRolesSubsystem::Deinitialize()
{
	if (UVPSettings* VPSettings = GetMutableDefault<UVPSettings>())
	{
		// If this is uninitialized before the VP Settings are destroyed, provide a fallback that returns an empty container.
		VPSettings->RolesGetter.BindStatic(&UVirtualProductionRolesSubsystem::GetRolesContainerFallback);
	}
}

TArray<FString> UVirtualProductionRolesSubsystem::GetActiveRoles() const
{
	TArray<FString> Roles;
	Algo::Transform(GetRolesContainer_Private(), Roles, [](const FGameplayTag& Tag) { return Tag.ToString(); });
	return Roles;
}

bool UVirtualProductionRolesSubsystem::HasActiveRole(const FString& Role) const
{
	constexpr bool bErrorWhenNotFound = false;
	const FGameplayTag Tag = UGameplayTagsManager::Get().RequestGameplayTag(*Role, bErrorWhenNotFound);
	if (Tag.IsValid())
	{
		return GetRolesContainer_Private().HasTag(Tag);
	}
	return false;
}

FString UVirtualProductionRolesSubsystem::GetActiveRolesString() const
{
	return GetRolesContainer_Private().ToStringSimple();
}

void UVirtualProductionRolesSubsystem::SetActiveRoles(const TArray<FString>& Roles)
{
	FGameplayTagContainer Container;
	
	for (const FString& Role : Roles)
	{
		constexpr bool bErrorIfNotFound = false;
		FGameplayTag Tag = FGameplayTag::RequestGameplayTag(*Role, bErrorIfNotFound);
		if (Tag.IsValid())
		{
			Container.AddTag(Tag);
		}
		else
		{
			UE_LOGF(LogVPRoles, Error, "Role %ls doesn't exist.", *Role);
		}
	}

	// Clear the transient roles if we're running through code while in editor.
	GetMutableDefault<UVPRolesUserSettings>()->Roles = MoveTemp(Container);

	BroadcastRolesChanged();
}

TArray<FString> UVirtualProductionRolesSubsystem::GetAllRoles() const
{
#if WITH_EDITOR
	// Get all roles from our collected sources.
	TArray<TSharedPtr<FGameplayTagNode>> Roles;
	for (FName RoleSource : GetRoleSources())
	{
		UGameplayTagsManager::Get().GetAllTagsFromSource(RoleSource, Roles);
	}

	TArray<FString> ReturnedRoles;
	ReturnedRoles.Reserve(Roles.Num());
	Algo::TransformIf(Roles, ReturnedRoles,
		[](const TSharedPtr<FGameplayTagNode>& Node){ return Node.IsValid(); },
		[](const TSharedPtr<FGameplayTagNode>& Node){ return Node->GetCompleteTagString(); });
	
	return ReturnedRoles;
#else
	return GetDefault<UVPRolesUserSettings>()->CachedRoles;
#endif
}

#if WITH_EDITOR
bool UVirtualProductionRolesSubsystem::AddRole(const FString& RoleName)
{
	const bool bResult = IGameplayTagsEditorModule::Get().AddNewGameplayTagToINI(RoleName, TEXT("VPRole"), DefaultVPRolesFileName);
	BroadcastRolesChanged();
	return bResult;
}
	
bool UVirtualProductionRolesSubsystem::RemoveRole(const FString& RoleName)
{
	bool bResult = false;
	if (const TSharedPtr<FGameplayTagNode> Node = UGameplayTagsManager::Get().FindTagNode(*RoleName))
	{
		bResult = IGameplayTagsEditorModule::Get().DeleteTagFromINI(Node);
		BroadcastRolesChanged();
	}
	return bResult;
}

#endif

bool UVirtualProductionRolesSubsystem::IsUsingCommandLineRoles() const
{
	return bUseCommandLineRoles; 
}

bool UVirtualProductionRolesSubsystem::HasCommandLineRoles() const
{
	return bHasCommandLineRoles;
}

void UVirtualProductionRolesSubsystem::UseCommandLineRoles(bool bInUseCommandLineRoles)
 {
	if (bHasCommandLineRoles)
	{
		bUseCommandLineRoles = bInUseCommandLineRoles;
		BroadcastRolesChanged();
	}
}

const TSet<FName>& UVirtualProductionRolesSubsystem::GetRoleSources() const
{
	return GetDefault<UVPRolesUserSettings>()->RoleSources;
}

FName UVirtualProductionRolesSubsystem::GetDefaultRoleSource() const
{
	return DefaultVPRolesFileName;
}

bool UVirtualProductionRolesSubsystem::HasActiveRole(const FGameplayTag& Role) const
{
	return GetRolesContainer_Private().HasTag(Role);
}

void UVirtualProductionRolesSubsystem::ImportCurrentRolesFromDeprecatedSetting()
{
	UVPSettings* DeprecatedSettings = GetMutableDefault<UVPSettings>();
	GetMutableDefault<UVPRolesUserSettings>()->Roles.AppendTags(DeprecatedSettings->Roles);
	DeprecatedSettings->Roles.Reset();
	DeprecatedSettings->SaveConfig();
}

const FGameplayTagContainer& UVirtualProductionRolesSubsystem::GetRolesContainer_Private() const
{
	if (bUseCommandLineRoles)
	{
		return *CommandLineRoles;
	}
		
	return GetMutableDefault<UVPRolesUserSettings>()->Roles;
}

const FGameplayTagContainer& UVirtualProductionRolesSubsystem::GetRolesContainerFallback()
{
	static const FGameplayTagContainer FallbackGetterContainer;
	return FallbackGetterContainer;
}

FGameplayTagContainer* UVirtualProductionRolesSubsystem::GetRolesContainerPtr(EGetRolesPtrSource Source)
{
	if (Source == EGetRolesPtrSource::Settings)
	{
		return &GetMutableDefault<UVPRolesUserSettings>()->Roles;
	}
	else
	{
		return CommandLineRoles.Get();
	}
}

void UVirtualProductionRolesSubsystem::ImportRoleSourcesFromVPSettings()
{
#if WITH_EDITOR
	const UVPSettings* VPSettings = GetDefault<UVPSettings>();
	for (const FGameplayTag& Tag : VPSettings->Roles)
	{
		FString DevComment;
		FName TagSource;
		bool bIsTagExplicit;
		bool bIsRestrictedTag;
		bool bAllowNonRestrictedChildren;
		if (UGameplayTagsManager::Get().GetTagEditorData(Tag.GetTagName(), DevComment, TagSource, bIsTagExplicit, bIsRestrictedTag, bAllowNonRestrictedChildren))
		{
			AddRoleSource(TagSource);
		}
	}
#endif
}

bool UVirtualProductionRolesSubsystem::ImportCurrentRolesFromCommandLine()
{
	FString ComandlineRoles;

	if (FParse::Value(FCommandLine::Get(), TEXT("-VPRole="), ComandlineRoles))
	{
		bHasCommandLineRoles = true;
		bUseCommandLineRoles = true;
		
		TArray<FString> RoleList;
		ComandlineRoles.ParseIntoArray(RoleList, TEXT("|"), true);

		for (const FString& Role : RoleList)
		{
			FGameplayTag Tag = FGameplayTag::RequestGameplayTag(*Role, false);
			if (Tag.IsValid())
			{
				CommandLineRoles->AddTag(Tag);
			}
			else
			{
				UE_LOGF(LogVPRoles, Error, "Role %ls doesn't exist.", *Role);
			}
		}
		return true;
	}
	return false;
}

void UVirtualProductionRolesSubsystem::BroadcastRolesChanged() const
{
#if WITH_EDITOR
	GetMutableDefault<UVPRolesUserSettings>()->CachedRoles = GetAllRoles();
	
	if (GEditor)
	{
		// Only save out the active roles if running in editor.
		GetMutableDefault<UVPRolesUserSettings>()->SaveConfig();
	}
#endif

	const TArray<FString> Roles = GetActiveRoles();
	OnRolesChangedNative.Broadcast(Roles);
	OnRolesChangedBP.Broadcast(Roles);

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Trigger a broadcast on the old system.
	GetMutableDefault<UVPSettings>()->OnRolesChanged.Broadcast();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

void UVirtualProductionRolesSubsystem::AddRoleSource(FName TagSource)
{	
	GetMutableDefault<UVPRolesUserSettings>()->RoleSources.Add(TagSource);
}

