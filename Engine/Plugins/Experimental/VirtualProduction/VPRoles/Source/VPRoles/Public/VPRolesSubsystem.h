// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Subsystems/EngineSubsystem.h"
#include "Templates/PimplPtr.h"

#include "VPRolesSubsystem.generated.h"

struct FGameplayTag;
struct FGameplayTagContainer;

/**
 * Subsystem for common Virtual Production roles operations.
 * The machine role(s) in a virtual production context read from the command line.
 * ie. "-VPRole=[Role.SubRole1|Role.SubRole2]"
 */
UCLASS()
class VPROLES_API UVirtualProductionRolesSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnRolesChanged, const TArray<FString>&, EnabledRoles);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRolesChangedNative, const TArray<FString>&);
	
public:
	//~ Begin USubsystem interface
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem interface 

	/**
	 * Get the currently active Virtual Production roles.
	 * @return The names of the current virtual production roles.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production|Roles")
	TArray<FString> GetActiveRoles() const;

	/**
	 * @return Whether a role is currently active.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production|Roles")
	bool HasActiveRole(const FString& Role) const;

	/**
	 * @return the currently active Virtual Production roles as a comma separated string representation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production|Roles")
	FString GetActiveRolesString() const;

	/**
	 * Set the current Virtual Production roles.
	 * @param Roles the list of roles to set as active.
	 * @note The roles must already have been added either through the UI or through AddRole.
	 * @note This will clear the previous current roles.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production|Roles")
	void SetActiveRoles(const TArray<FString>& Roles);

	/**
	 * Get all available roles that can be set as current.
	 * @return The available roles.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production|Roles")
	TArray<FString> GetAllRoles() const;
	
#if WITH_EDITOR
	/**
	 * Add a new virtual production role.
	 * @param RoleName the name of the role to add.
	 * @return Whether the operation succeeded.
	 * @note This will attempt to modify the underlying VProles.ini file so it must be checked out or made writable or the operation will fail.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production|Roles")
	bool AddRole(const FString& RoleName);

	/**
	 * Remove a virtual production role.
	 * @param RoleName the name of the role to remove.
	 * @return Whether the operation succeeded.
	 * @note This will attempt to modify the underlying VProles.ini file so it must be checked out or made writable or the operation will fail.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Production|Roles")
	bool RemoveRole(const FString& RoleName);
#endif

	/**
	 * Delegate called when the roles are modified, ie. role is added, current roles are replaced, etc.
	 */
	FOnRolesChangedNative& OnRolesChanged()
	{
		return OnRolesChangedNative;
	}

	/**
	 * @return Whether we are using the roles passed in command line if any.  
	 */
	bool IsUsingCommandLineRoles() const;

	/**
	 * @return Whether roles were passed through the command line.
	 */
	bool HasCommandLineRoles() const;

	/**
	 * Set whether the command line roles should be used if any were specified.
	 */
	void UseCommandLineRoles(bool bUseCommandLineRoles);

	/**
	 * @return The list of sources that provide VP roles.
	 */
	const TSet<FName>& GetRoleSources() const;
	
	/**
	 * @return The name of the default VP role ini file.
	 */
	FName GetDefaultRoleSource() const;

	/**
	 * @return Whether a role is currently active.
	 */
	bool HasActiveRole(const FGameplayTag& Role) const;
	
	/**
	 * Get the roles container, either from the command line or from the VP Roles settings.
	 * @note New code should not use this method, it is only provided for backwards compatibility.
	 */
	const FGameplayTagContainer& GetRolesContainer_Private() const;
public:
	/**
	 * Delegate called when the roles are modified, ie. role is added, current roles are replaced, etc.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Virtual Production|Roles", meta = (ScriptName ="OnRolesChanged"))
	FOnRolesChanged OnRolesChangedBP;
	
private:
	/** Save roles to the VP Role config file and trigger a broadcast on the OnRolesChanged delegates. */
	void BroadcastRolesChanged() const;
	/** Add a role source to the VP Roles settings. Used to keep track of which config files provided VP Roles. */
	void AddRoleSource(FName TagSource);
	/** Parses command line roles. */
	bool ImportCurrentRolesFromCommandLine();
	/** Imports VP Roles from the VP Settings. */
	void ImportCurrentRolesFromDeprecatedSetting();
	/** Fallback handler that returns an empty tag container. */
	static const FGameplayTagContainer& GetRolesContainerFallback();
	enum class EGetRolesPtrSource
	{
		Settings,
		CommandLine
	};
	/** Get a pointer to the roles gameplay tag container */
	FGameplayTagContainer* GetRolesContainerPtr(EGetRolesPtrSource Source);
	/** Gather all gameplay tag sources from the tags that were imported from the VP settings. */
	void ImportRoleSourcesFromVPSettings();

private:
	/** Delegate called when the roles are modified, ie. role is added, current roles are replaced, etc. */
	FOnRolesChangedNative OnRolesChangedNative;
	/** Holds the role container populated from the roles specified on the command line. */
	TPimplPtr<FGameplayTagContainer> CommandLineRoles;
	/** Whether command line roles should be used. */
	bool bUseCommandLineRoles = false;
	/** Whether command line roles were specified.  */
	bool bHasCommandLineRoles = false;
	
	friend class FVPRolesModule;
	friend class FVPRolesEditorModule;
};
