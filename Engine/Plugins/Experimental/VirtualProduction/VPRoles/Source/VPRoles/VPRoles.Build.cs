// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VPRoles : ModuleRules
{
	public VPRoles(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"GameplayTags",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"VPSettings"
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"GameplayTagsEditor",
					"UnrealEd",
				}
			);
		}
	}
}
