// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPRolesEditorModule.h"

#include "Framework/Application/SlateApplication.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsEditorModule.h"
#include "GameplayTagsSettings.h"
#include "LevelEditor.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/MessageDialog.h"
#include "SGameplayTagWidget.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "VPRolesEditorStyle.h"
#include "VPRolesSubsystem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"


DEFINE_LOG_CATEGORY(LogVPRolesEditor)
#define LOCTEXT_NAMESPACE "VPRolesEditor"

namespace UE::Private::VPRolesEditor
{
	SGameplayTagWidget::ETagFilterResult OnFilterTag(const TSharedPtr<FGameplayTagNode>& InGameplayTagNode)
	{
		if (GEngine)
		{
			const TSet<FName>& TagSources = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>()->GetRoleSources();
			
			FString DevComment;
			FName TagSource;
			bool bIsExplicit = false;
			bool bIsRestricted = false;
			bool bAllowNonRestrictedChildren = false;

			static const FString VPRoleComment = TEXT("VPRole");
			
			UGameplayTagsManager::Get().GetTagEditorData(InGameplayTagNode->GetCompleteTagName(), DevComment, TagSource, bIsExplicit, bIsRestricted, bAllowNonRestrictedChildren);
			
			if ((TagSources.Contains(TagSource) || DevComment == VPRoleComment))
			{
				return SGameplayTagWidget::ETagFilterResult::IncludeTag;
			}
		}
		
		return SGameplayTagWidget::ETagFilterResult::ExcludeTag;
	};
	
}

void FVPRolesEditorModule::StartupModule()
{
	FVPRolesEditorStyle::Initialize();
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FVPRolesEditorModule::ExtendLevelEditorToolbar);
}

void FVPRolesEditorModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FVPRolesEditorStyle::Shutdown();
}

void FVPRolesEditorModule::ExtendLevelEditorToolbar()
{
	if (UVirtualProductionRolesSubsystem* RolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>())
	{
		if (UGameplayTagsSettings* GameplayTagsSettings = GetMutableDefault<UGameplayTagsSettings>())
		{
			GameplayTagsSettings->ImportTagsFromConfig = true;
		}
		
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");

		FToolMenuEntry VPRolesEntry = FToolMenuEntry::InitComboButton(
			"VPRolesMenu",
			FUIAction(),
			FOnGetContent::CreateRaw(this, &FVPRolesEditorModule::GenerateVPRolesLevelEditorToolbarMenu),
			LOCTEXT("LevelEditorToolbarVPRolesButtonLabel", "VP Roles"),
			LOCTEXT("LevelEditorToolbarVPRolesButtonTooltip", "Edit VP Roles"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Role"));
		
		VPRolesEntry.StyleNameOverride = "CalloutToolbar";
		Menu->FindOrAddSection("VPRoles").AddEntry(VPRolesEntry);
	}
}

TSharedRef<SWidget> FVPRolesEditorModule::GenerateVPRolesLevelEditorToolbarMenu()
{
	bShowAddRoleTextBox = false;
	UVirtualProductionRolesSubsystem* RolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>();
	if (!RolesSubsystem)
	{
		return SNullWidget::NullWidget;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FUICommandList> CommandBindings = LevelEditorModule.GetGlobalLevelEditorActions();

	constexpr bool bShouldCloseMenuAfterSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseMenuAfterSelection, CommandBindings);

	constexpr bool bNoIndent = false;
	constexpr bool bSearchable = false;
	static const FName NoExtensionHook = NAME_None;
	
	{
		MenuBuilder.BeginSection(NoExtensionHook, LOCTEXT("ModifyRolesSection", "Modify Roles"));
		FMenuEntryParams MenuEntryParams;

		FUIAction AddRoleAction;
		AddRoleAction.ExecuteAction = FExecuteAction::CreateLambda([this]()
		{
			if (TSharedPtr<SWidget> AddRoleWidget = AddRoleEditableBoxWeakPtr.Pin())
			{
				bShowAddRoleTextBox = true;
				FSlateApplication::Get().SetKeyboardFocus(AddRoleWidget);
			}
			return;
		});
		
		AddRoleAction.CanExecuteAction = FCanExecuteAction::CreateLambda([this, RolesSubsystem]()
		{
			return !bShowAddRoleTextBox && !RolesSubsystem->IsUsingCommandLineRoles();
		});

		MenuBuilder.AddMenuEntry(LOCTEXT("AddRoleEntry", "Add Role"),
			LOCTEXT("AddRoleToolTip", "Add a new virtual production role."),
			FSlateIcon(FVPRolesEditorStyle::Get().GetStyleSetName(), "VPRolesEditor.AddRole"),
			AddRoleAction);
		
		MenuBuilder.EndSection();
	}
	
	{
		MenuBuilder.BeginSection(NoExtensionHook, LOCTEXT("RolesSection", "Roles"));
		MenuBuilder.AddWidget(GenerateGameplayTagWidget(), FText::GetEmpty(), bNoIndent, bSearchable);
		MenuBuilder.EndSection();
	}
	
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FVPRolesEditorModule::GenerateGameplayTagWidget()
{
	UVirtualProductionRolesSubsystem* RolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>();
	if (!RolesSubsystem)
	{
		return SNullWidget::NullWidget;
	}
	
	if (DefaultTagSourceConfigName.IsEmpty())
	{
		DefaultTagSourceConfigName = RolesSubsystem->GetDefaultRoleSource().ToString();
	}
	
	static FSlateColorBrush TransparentBrush{ FLinearColor::Transparent };
	
	TArray<SGameplayTagWidget::FEditableGameplayTagContainerDatum> EditableContainers;
	FGameplayTagContainer* ContainerPtr = RolesSubsystem->GetRolesContainerPtr(UVirtualProductionRolesSubsystem::EGetRolesPtrSource::Settings);
	EditableContainers.Emplace(nullptr, ContainerPtr);

	TArray<SGameplayTagWidget::FEditableGameplayTagContainerDatum> CommandLineEditableContainers;
    FGameplayTagContainer* CommandLineContainerPtr = RolesSubsystem->GetRolesContainerPtr(UVirtualProductionRolesSubsystem::EGetRolesPtrSource::CommandLine);
    CommandLineEditableContainers.Emplace(nullptr, CommandLineContainerPtr);

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(FMargin(8.f, 0.f))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(2.f)
			.AutoHeight()
			[
				SNew(SBorder)
				.Visibility_Lambda([RolesSubsystem]() { return RolesSubsystem->HasCommandLineRoles() ? EVisibility::Visible : EVisibility::Collapsed; })
				.Padding(FMargin(6.f, 4.f))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([RolesSubsystem](){ return RolesSubsystem->IsUsingCommandLineRoles() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
						.OnCheckStateChanged_Lambda([RolesSubsystem, this](ECheckBoxState CheckBoxState)
						{
							RolesSubsystem->UseCommandLineRoles(CheckBoxState == ECheckBoxState::Checked ? true : false);
						})
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)	
						.Text(LOCTEXT("UseCommandLineRolesLabel", "Use command line roles"))
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.Visibility_Lambda(	[RolesSubsystem, this](){ return !bShowAddRoleTextBox || RolesSubsystem->IsUsingCommandLineRoles() ? EVisibility::Collapsed : EVisibility::Visible; })
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					// To match the padding of the gameplay tag list.
					SNew(SImage)
					.Image(FAppStyle::GetBrush("TreeArrow_Collapsed"))
					.Visibility(EVisibility::Hidden)
				]
				+SHorizontalBox::Slot()
	            .AutoWidth()
	            [
	                SNew(SCheckBox)
	                .IsEnabled(false)
	            ]
				+SHorizontalBox::Slot()
				.Padding(0.f)
				[
					SAssignNew(AddRoleEditableBoxWeakPtr, SEditableTextBox)
						.IsReadOnly_Lambda([RolesSubsystem](){ return RolesSubsystem->IsUsingCommandLineRoles(); })
						.ClearKeyboardFocusOnCommit(false)
						.HintText(LOCTEXT("GameplayTagWidget_AddRole", "Add Role"))
						.OnTextChanged_Raw(this, &FVPRolesEditorModule::OnAddTagTextChanged)
						.OnTextCommitted_Raw( this, &FVPRolesEditorModule::OnSubmitNewTag)
				]
			]
			+ SVerticalBox::Slot()
			.Padding(0.f)
			.AutoHeight()
			[
				SNew(SBox)
				.MinDesiredWidth(200.f)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex_Lambda([RolesSubsystem]()
					{
						return RolesSubsystem->IsUsingCommandLineRoles() ? 1 : 0;
					})
					+SWidgetSwitcher::Slot()
					[
						SNew(SGameplayTagWidget, EditableContainers)
							.Padding(0.f)
							.OnFilterTag_Static(&UE::Private::VPRolesEditor::OnFilterTag)
							.ForceHideAddNewTag(true)
							.ForceHideAddNewTagSource(true)
							.ForceHideTagTreeControls(true)
							.GameplayTagUIMode(EGameplayTagUIMode::HybridMode)
							.MaxHeight(0) // Height of 0 means the height is not limited.
							.BackgroundBrush(&TransparentBrush)
							.TagTreeViewBackgroundBrush(&TransparentBrush)
							.OnTagChanged_Lambda([RolesSubsystem]{ RolesSubsystem->BroadcastRolesChanged(); })
					]
					+SWidgetSwitcher::Slot()
					[
						SNew(SGameplayTagWidget, CommandLineEditableContainers)
							.Padding(0.f)
							.OnFilterTag_Static(&UE::Private::VPRolesEditor::OnFilterTag)
							.ForceHideAddNewTag(true)
							.ForceHideAddNewTagSource(true)
							.ForceHideTagTreeControls(true)
							.GameplayTagUIMode(EGameplayTagUIMode::SelectionMode)
							.MaxHeight(0) // Height of 0 means the height is not limited.
							.BackgroundBrush(&TransparentBrush)
							.TagTreeViewBackgroundBrush(&TransparentBrush)
							.ReadOnly(true)
					]					
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSettingsEditorCheckoutNotice)
				.Visibility_Raw(this, &FVPRolesEditorModule::DetermineFileWatcherWidgetVisibility)
				.ConfigFilePath(RolesSubsystem->GetDefaultRoleSource().ToString())
			]
		];
}

void FVPRolesEditorModule::OnAddTagTextChanged(const FText& Text)
{
	if (const TSharedPtr<SEditableTextBox> AddTagEditableBox = AddRoleEditableBoxWeakPtr.Pin())
	{
		const FString TagName = Text.ToString();
		FText ErrorMsg;

		if (Text.IsEmpty())
		{
			AddTagEditableBox->SetError(FText::GetEmpty());
			return;
		}

		if (!UGameplayTagsManager::Get().IsValidGameplayTagString(TagName, &ErrorMsg))
		{
			FText MessageTitle(LOCTEXT("InvalidTag", "Invalid Tag"));
			AddTagEditableBox->SetError(ErrorMsg);
		}
		else
		{
			AddTagEditableBox->SetError(FText::GetEmpty());
		}
	}
}

EVisibility FVPRolesEditorModule::DetermineFileWatcherWidgetVisibility() const
{
	return CanModifyTagSource() ? EVisibility::Collapsed : EVisibility::Visible;
}

bool FVPRolesEditorModule::CanModifyTagSource() const
{
	if (!DefaultTagSourceConfigName.IsEmpty())
	{
		return !FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*DefaultTagSourceConfigName);
	}
	else
	{
		ensureMsgf(false, TEXT("File %s should already exist."), *DefaultTagSourceConfigName);
		return false;
	}
}

void FVPRolesEditorModule::OnSubmitNewTag(const FText& CommittedText, ETextCommit::Type CommitType)
{
	if (!GEngine)
	{
		return;
	}
	
	const TSharedPtr<SEditableTextBox> AddTagEditableBox = AddRoleEditableBoxWeakPtr.Pin();
	if (!AddTagEditableBox)
	{
		return;
	}
	
	if (const UVirtualProductionRolesSubsystem* RolesSubsystem = GEngine->GetEngineSubsystem<UVirtualProductionRolesSubsystem>())
	{
		if (CommitType == ETextCommit::Type::OnEnter)
		{
			UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

			// Only support adding tags via ini file
			if (Manager.ShouldImportTagsFromINI() == false)
			{
				return;
			}

			const FText TagNameAsText = CommittedText;
			const FString TagName = TagNameAsText.ToString();

			if (TagName.IsEmpty())
			{
				return;
			}

			if (DefaultTagSourceConfigName.IsEmpty())
			{
				return;
			}

			// Check to see if this is a valid tag
			// First check the base rules for all tags then look for any additional rules in the delegate
			FText ErrorMsg;
			if (!Manager.IsValidGameplayTagString(TagName, &ErrorMsg))
			{
				AddTagEditableBox->SetError(ErrorMsg);
				return;
			}

			bool bCanAddTag = false;
			if (!CanModifyTagSource())
			{
				if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("SaveAsDefaultsIsReadOnlyMessage", "The ini file for these settings is not currently writable. Would you like to make it writable?")) == EAppReturnType::Yes)
				{
					bCanAddTag = SettingsHelpers::MakeWritable(DefaultTagSourceConfigName, true);
				}
			}
			else
			{
				bCanAddTag = true;
				AddTagEditableBox->SetText(FText::GetEmpty());
			}

			if (bCanAddTag)
			{
				if (IGameplayTagsEditorModule::Get().AddNewGameplayTagToINI(TagName, FString(), *DefaultTagSourceConfigName))
				{
					if (TSharedPtr<FGameplayTagNode> Node = Manager.FindTagNode(*TagName))
					{
						RolesSubsystem->BroadcastRolesChanged();
					}
				}
				
				bShowAddRoleTextBox = false;
			}
		}

		if (CommitType == ETextCommit::Type::OnUserMovedFocus)
		{
			bShowAddRoleTextBox = false;
			AddTagEditableBox->SetText(FText::GetEmpty());
		}
	}
}

IMPLEMENT_MODULE(FVPRolesEditorModule, VPRolesEditor)

#undef LOCTEXT_NAMESPACE /*VPRolesEditor*/
