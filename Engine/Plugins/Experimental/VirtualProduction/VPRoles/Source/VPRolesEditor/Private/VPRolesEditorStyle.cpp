// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPRolesEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SlateGameResources.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"

TSharedPtr<FSlateStyleSet> FVPRolesEditorStyle::StyleInstance;

void FVPRolesEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FVPRolesEditorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FVPRolesEditorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("VPRolesEditorStyle"));
	return StyleSetName;
}

#define IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( Style->RootToContentDir( RelativePath, TEXT(".svg") ), __VA_ARGS__ )

const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

TSharedRef<FSlateStyleSet> FVPRolesEditorStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShared<FSlateStyleSet>(FVPRolesEditorStyle::GetStyleSetName());
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("VirtualProductionUtilities")->GetBaseDir() / TEXT("Resources"));

	FSlateBrush AddRoleBrush = *FAppStyle::Get().GetBrush("Icons.Plus");
	AddRoleBrush.TintColor = FStyleColors::AccentGreen;

	Style->Set("VPRolesEditor.AddRole", new FSlateBrush(AddRoleBrush));
	return Style;
}

const ISlateStyle& FVPRolesEditorStyle::Get()
{
	static const FVPRolesEditorStyle Inst;
	return *StyleInstance;
}

#undef IMAGE_BRUSH_SVG
