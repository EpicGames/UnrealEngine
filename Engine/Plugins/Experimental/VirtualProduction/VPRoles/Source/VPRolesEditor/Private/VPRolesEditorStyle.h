// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

class FVPRolesEditorStyle
{
public:
	/** Initialize the style. */
	static void Initialize();
	/** Unregister the style. */
	static void Shutdown();
	/** The Slate style set for VP Roles */
	static const ISlateStyle& Get();
	/** Get the style set name. */
	static FName GetStyleSetName();

private:
	/** Initializes the singleton and sets the buttons used by the vp roles widget.  */
	static TSharedRef<class FSlateStyleSet> Create();

private:
	/** Singleton instance of the style. */
	static TSharedPtr<class FSlateStyleSet> StyleInstance;
};