// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Types/SlateEnums.h"

class SWidget;
struct EVisibility;

DECLARE_LOG_CATEGORY_EXTERN(LogVPRolesEditor, Log, Log);

class VPROLESEDITOR_API FVPRolesEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~End IModuleInterface

private:
	/** Core delegate handler for extending the level editor toolbar. */
	void ExtendLevelEditorToolbar();
	
	/** Generates the toolbar button widget. */
	TSharedRef<SWidget> GenerateVPRolesLevelEditorToolbarMenu();

	/** Generate the gameplay tag widget. */
	TSharedRef<SWidget> GenerateGameplayTagWidget();
	
	/** Callback for when a new tag is added */
	void OnSubmitNewTag(const FText& CommittedText, ETextCommit::Type CommitType);

	/** Callback for when the add tag textbox's text changed. */
	void OnAddTagTextChanged(const FText& Text);
	
	/** Whether the file watcher widget should be shown for the default tag source config file. */
	EVisibility DetermineFileWatcherWidgetVisibility() const;
	
	/** Whether the default VP role tag source can be modified. */
	bool CanModifyTagSource() const;

private:
	/** The file name for the default vp role tag source. */
	FString DefaultTagSourceConfigName;
	/** Holds the widget generated by the GameplayTags editor module. */
	TWeakPtr<class SGameplayTagWidget> GameplayTagsWidget;
	/** Holds the editable text box widget, used to update the text box's error message. */
	TWeakPtr<class SEditableTextBox> AddRoleEditableBoxWeakPtr;
	/** Whether to show the add role textbox. */
	bool bShowAddRoleTextBox = false;
};