// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VPRolesEditor : ModuleRules
{
	public VPRolesEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"GameplayTags",
				"GameplayTagsEditor",
				"LevelEditor",
				"Projects",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"VPRoles",
				"SharedSettingsWidgets"
			}
		);
	}
}
