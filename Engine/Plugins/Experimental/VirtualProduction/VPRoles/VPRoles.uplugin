{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Virtual Production Roles",
	"Description": "Allows users to manage Virtual Production Role assignment.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Hidden" : true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "VPRoles",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "VPRolesEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
	],
	"Plugins": [
		{
			"Name": "VPSettings",
			"Enabled": true
		},
		{
			"Name": "GameplayTagsEditor",
			"Enabled": true
		}
	]
}
